// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CookOnTheFlyNetServer : ModuleRules
{
    public CookOnTheFlyNetServer(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "TargetPlatform"
                });

        PublicDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "Sockets",
                "Networking",
                "CookOnTheFly"
            }
        );
    }
}
