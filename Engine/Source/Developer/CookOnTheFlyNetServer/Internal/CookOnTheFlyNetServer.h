// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Async/Future.h"
#include "Misc/Timespan.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "CookOnTheFly.h"

class FInternetAddr;

DECLARE_LOG_CATEGORY_EXTERN(LogCookOnTheFlyNetworkServer, Log, All);

namespace UE { namespace Cook
{

enum class ECookOnTheFlyNetworkServerProtocol : uint32
{
	Tcp,
	Platform
};

struct FCookOnTheFlyNetworkServerOptions
{
	/* Server protocol*/
	ECookOnTheFlyNetworkServerProtocol Protocol = ECookOnTheFlyNetworkServerProtocol::Tcp;

	/* The port number to bind to (-1 = default port, 0 = any available port) */
	int32 Port = INDEX_NONE;

	/* Active target platform(s) */
	TArray<ITargetPlatform*> TargetPlatforms;
};

class ICookOnTheFlyClientConnection
{
public:
	virtual ~ICookOnTheFlyClientConnection() = default;

	virtual FName GetPlatformName() const = 0;
	virtual const ITargetPlatform* GetTargetPlatform() const = 0;
	virtual bool GetIsSingleThreaded() const = 0;
	virtual bool SendMessage(const FCookOnTheFlyMessage& Message) = 0;
	virtual void SetZenInfo(const FString& InProjectId, const FString& InOplogId, const FString& InHostName, uint16 InHostPort) = 0;
};

class ICookOnTheFlyNetworkServer
{
public:
	virtual ~ICookOnTheFlyNetworkServer() = default;

	DECLARE_EVENT_OneParam(ICookOnTheFlyNetworkServer, FClientConnectionEvent, ICookOnTheFlyClientConnection&);
	virtual FClientConnectionEvent& OnClientConnected() = 0;
	virtual FClientConnectionEvent& OnClientDisconnected() = 0;

	DECLARE_DELEGATE_RetVal_TwoParams(bool, FHandleRequestDelegate, ICookOnTheFlyClientConnection&, const FCookOnTheFlyRequest&);
	virtual FHandleRequestDelegate& OnRequest(ECookOnTheFlyMessage MessageType) = 0;

	virtual bool Start() = 0;
	virtual bool IsReadyToAcceptConnections(void) const = 0;
	virtual bool GetAddressList(TArray<TSharedPtr<FInternetAddr>>& OutAddresses) const = 0;
	virtual FString GetSupportedProtocol() const = 0;
	virtual int32 NumConnections() const = 0;
};

class ICookOnTheFlyNetworkServerModule
	: public IModuleInterface
{
public:
	virtual ~ICookOnTheFlyNetworkServerModule() = default;

	virtual TSharedPtr<ICookOnTheFlyNetworkServer> CreateServer(const FCookOnTheFlyNetworkServerOptions& Options) = 0;
};

}} // namespace UE::Cook
