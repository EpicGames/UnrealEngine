// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookOnTheFlyNetServer.h"
#include "Modules/ModuleManager.h"
#include "TCPServer.h"
#include "PlatformProtocolServer.h"

DEFINE_LOG_CATEGORY(LogCookOnTheFlyNetworkServer);

class FCookOnTheFlyNetworkServerModule final
	: public UE::Cook::ICookOnTheFlyNetworkServerModule
{
public:
	virtual TSharedPtr<UE::Cook::ICookOnTheFlyNetworkServer> CreateServer(const UE::Cook::FCookOnTheFlyNetworkServerOptions& Options) override
	{
		using namespace UE::Cook;

		switch (Options.Protocol)
		{
		case ECookOnTheFlyNetworkServerProtocol::Tcp:
			return MakeShared<FCookOnTheFlyServerTCP>(Options.Port, Options.TargetPlatforms);
		case ECookOnTheFlyNetworkServerProtocol::Platform:
			return MakeShared<FCookOnTheFlyServerPlatformProtocol>(Options.TargetPlatforms);
		default:
			UE_LOGF(LogCookOnTheFlyNetworkServer, Fatal, "Unsupported protocol: %d", int(Options.Protocol));
			return nullptr;
		}
	}
};

IMPLEMENT_MODULE(FCookOnTheFlyNetworkServerModule, CookOnTheFlyNetServer);
