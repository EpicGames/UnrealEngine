// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookOnTheFlyNetServerBase.h"
#include "Serialization/BufferArchive.h"
#include "Interfaces/ITargetPlatform.h"
#include "Serialization/ArrayReader.h"
#include "HAL/RunnableThread.h"
#include "SocketSubsystem.h"
#include "IPAddress.h"

FCookOnTheFlyClientConnectionBase::FCookOnTheFlyClientConnectionBase(FCookOnTheFlyNetworkServerBase& InOwner)
	: Owner(InOwner)
{
}

FCookOnTheFlyClientConnectionBase::~FCookOnTheFlyClientConnectionBase()
{
	if (WorkerThread)
	{
		delete WorkerThread;
	}
	if (bClientConnectedBroadcasted && Owner.ClientDisconnectedEvent.IsBound())
	{
		Owner.ClientDisconnectedEvent.Broadcast(*this);
	}
}

bool FCookOnTheFlyClientConnectionBase::Initialize()
{
	FArrayReader HandshakeRequestPayload;
	if (!ReceivePayload(HandshakeRequestPayload))
	{
		UE_LOGF(LogCookOnTheFlyNetworkServer, Warning, "Failed to receive handshake payload");
		return false;
	}

	TArray<FString> TargetPlatformNames;
	HandshakeRequestPayload << TargetPlatformNames;
	HandshakeRequestPayload << bIsSingleThreaded;

	bool bIsOk = true;
	FString PlatformNameString;
	if (!TargetPlatformNames.IsEmpty())
	{
		// figure out the best matching target platform for the set of valid ones
		for (int32 TPIndex = 0; TPIndex < TargetPlatformNames.Num() && !TargetPlatform; TPIndex++)
		{
			UE_LOGF(LogCookOnTheFlyNetworkServer, Verbose, "    Possible Target Platform from client: %ls", *TargetPlatformNames[TPIndex]);

			// look for a matching target platform
			for (int32 ActiveTPIndex = 0; ActiveTPIndex < Owner.ActiveTargetPlatforms.Num(); ActiveTPIndex++)
			{
				UE_LOGF(LogCookOnTheFlyNetworkServer, Verbose, "   Checking against: %ls", *Owner.ActiveTargetPlatforms[ActiveTPIndex]->PlatformName());
				if (Owner.ActiveTargetPlatforms[ActiveTPIndex]->PlatformName() == TargetPlatformNames[TPIndex])
				{
					TargetPlatform = Owner.ActiveTargetPlatforms[ActiveTPIndex];
					PlatformNameString = TargetPlatform->PlatformName();
					PlatformName = FName(PlatformNameString);
					break;
				}
			}
		}

		// if we didn't find one, reject client and also print some warnings
		if (!TargetPlatform)
		{
			// reject client we can't cook/compile shaders for you!
			UE_LOGF(LogCookOnTheFlyNetworkServer, Warning, "Unable to find target platform for client, terminating client connection!");

			for (int32 TPIndex = 0; TPIndex < TargetPlatformNames.Num(); TPIndex++)
			{
				UE_LOGF(LogCookOnTheFlyNetworkServer, Warning, "    Target platforms from client: %ls", *TargetPlatformNames[TPIndex]);
			}
			for (int32 ActiveTPIndex = 0; ActiveTPIndex < Owner.ActiveTargetPlatforms.Num(); ActiveTPIndex++)
			{
				UE_LOGF(LogCookOnTheFlyNetworkServer, Warning, "    Active target platforms on server: %ls", *Owner.ActiveTargetPlatforms[ActiveTPIndex]->PlatformName());
			}
			bIsOk = false;
		}
	}

	if (!bIsOk)
	{
		return false;
	}

	if (Owner.ClientConnectedEvent.IsBound())
	{
		Owner.ClientConnectedEvent.Broadcast(*this);
		bClientConnectedBroadcasted = true;
	}

	FBufferArchive HandshakeResponsePayload;
	HandshakeResponsePayload << bIsOk;
	HandshakeResponsePayload << PlatformNameString;
	HandshakeResponsePayload << ZenProjectId;
	HandshakeResponsePayload << ZenHostName;
	HandshakeResponsePayload << ZenHostPort;

	if (ZenHostName == "localhost")
	{
		TArray<FString> RemoteHostNames;
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
		if (SocketSubsystem != nullptr)
		{
			TArray<TSharedPtr<FInternetAddr>> Addresses;
			if (SocketSubsystem->GetLocalAdapterAddresses(Addresses))
			{
				for (const TSharedPtr<FInternetAddr>& Address : Addresses)
				{
					RemoteHostNames.Add(Address->ToString(false));
				}
			}
		}
		int32 RemoteHostNameCount = RemoteHostNames.Num();
		HandshakeResponsePayload << RemoteHostNameCount;
		for (FString RemoteHostName : RemoteHostNames)
		{
			HandshakeResponsePayload << RemoteHostName;
		}
	}

	if (!SendPayload(HandshakeResponsePayload))
	{
		return false;
	}

#if UE_BUILD_DEBUG
	// this thread needs more space in debug builds as it tries to log messages and such
	const static uint32 ThreadSize = 2 * 1024 * 1024;
#else
	const static uint32 ThreadSize = 1 * 1024 * 1024;
#endif
	WorkerThread = FRunnableThread::Create(this, TEXT("FCookOnTheFlyClientConnection"), ThreadSize, TPri_AboveNormal);

	return true;
}

uint32 FCookOnTheFlyClientConnectionBase::Run()
{
	while (!bStopRequested)
	{
		// read a header and payload pair
		FArrayReader Payload;
		if (!ReceivePayload(Payload))
		{
			UE_LOGF(LogCookOnTheFlyNetworkServer, Warning, "ReceivePayload failed");
			break;
		}
		ProcessPayload(Payload);
	}
	return 0;
}


bool FCookOnTheFlyClientConnectionBase::ProcessPayload(FArchive& Payload)
{
	using namespace UE::Cook;

	FCookOnTheFlyRequest Request;
	Payload << Request;

	return Owner.ProcessRequest(*this, Request);
}

bool FCookOnTheFlyClientConnectionBase::SendMessage(const UE::Cook::FCookOnTheFlyMessage& Message)
{
	FBufferArchive Payload;
	Payload.Reserve(IntCastChecked<int32>(Message.TotalSize()));
	Payload << const_cast<UE::Cook::FCookOnTheFlyMessage&>(Message);
	return SendPayload(Payload);
}

FCookOnTheFlyNetworkServerBase::FCookOnTheFlyNetworkServerBase(const TArray<ITargetPlatform*>& InActiveTargetPlatforms)
	: ActiveTargetPlatforms(InActiveTargetPlatforms)
{

}

bool FCookOnTheFlyNetworkServerBase::ProcessRequest(FCookOnTheFlyClientConnectionBase& Connection, const UE::Cook::FCookOnTheFlyRequest& Request)
{
	using namespace UE::Cook;

	ECookOnTheFlyMessage RequestType = Request.GetMessageType();
	FHandleRequestDelegate* Handler = Handlers.Find(RequestType);
	if (!Handler)
	{
		return false;
	}
	if (!Handler->IsBound())
	{
		return false;
	}
	return Handler->Execute(Connection, Request);
}

UE::Cook::ICookOnTheFlyNetworkServer::FHandleRequestDelegate& FCookOnTheFlyNetworkServerBase::OnRequest(UE::Cook::ECookOnTheFlyMessage MessageType)
{
	FHandleRequestDelegate& Handler = Handlers.FindOrAdd(MessageType);
	return Handler;
}
