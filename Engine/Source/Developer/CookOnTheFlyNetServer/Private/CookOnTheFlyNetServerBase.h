// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CookOnTheFlyNetServer.h"
#include "HAL/Runnable.h"

class FCookOnTheFlyNetworkServerBase;

class FCookOnTheFlyClientConnectionBase
	: public UE::Cook::ICookOnTheFlyClientConnection
	, private FRunnable
{
public:
	FCookOnTheFlyClientConnectionBase(FCookOnTheFlyNetworkServerBase& InOwner);
	virtual ~FCookOnTheFlyClientConnectionBase();

	FName GetPlatformName() const override
	{
		return PlatformName;
	}

	const ITargetPlatform* GetTargetPlatform() const
	{
		return TargetPlatform;
	}

	bool GetIsSingleThreaded() const
	{
		return bIsSingleThreaded;
	}

	void SetZenInfo(const FString& InProjectId, const FString& InOplogId, const FString& InHostName, uint16 InHostPort)
	{
		ZenProjectId = InProjectId;
		check(InOplogId == PlatformName);
		ZenHostName = InHostName;
		ZenHostPort = InHostPort;
	}

	bool Initialize();

	bool SendMessage(const UE::Cook::FCookOnTheFlyMessage& Message) override;

	bool IsRunning()
	{
		return bRunning;
	}

	void Disconnect()
	{
		Stop();
	}

private:
	virtual void OnInit() {}

	virtual bool Init() override
	{
		OnInit();
		return true;
	}

	virtual uint32 Run() override;

	virtual void Stop() override
	{
		bStopRequested = true;
	}

	virtual void OnExit() {}

	virtual void Exit() override
	{
		OnExit();
		bRunning = false;
	}

	bool ProcessPayload(FArchive& Payload);
	virtual bool ReceivePayload(FArrayReader& Payload) = 0;
	virtual bool SendPayload(const TArray<uint8>& Out) = 0;

	FCookOnTheFlyNetworkServerBase& Owner;
	FName PlatformName;
	ITargetPlatform* TargetPlatform = nullptr;
	bool bIsSingleThreaded = false;
	TAtomic<bool> bStopRequested{ false };
	TAtomic<bool> bRunning{ true };
	FRunnableThread* WorkerThread = nullptr;
	bool bClientConnectedBroadcasted = false;

	FString ZenProjectId;
	FString ZenHostName;
	uint16 ZenHostPort = 0;
};

class FCookOnTheFlyNetworkServerBase
	: public UE::Cook::ICookOnTheFlyNetworkServer
{
	friend class FCookOnTheFlyClientConnectionBase;

public:
	FCookOnTheFlyNetworkServerBase(const TArray<ITargetPlatform*>& InActiveTargetPlatforms);

	bool ProcessRequest(FCookOnTheFlyClientConnectionBase& Connection, const UE::Cook::FCookOnTheFlyRequest& Request);

	DECLARE_DERIVED_EVENT(FCookOnTheFlyNetworkServerBase, UE::Cook::ICookOnTheFlyNetworkServer::FClientConnectionEvent, FClientConnectionEvent);
	FClientConnectionEvent& OnClientConnected() override
	{
		return ClientConnectedEvent;
	}

	FClientConnectionEvent& OnClientDisconnected() override
	{
		return ClientDisconnectedEvent;
	}

	FHandleRequestDelegate& OnRequest(UE::Cook::ECookOnTheFlyMessage MessageType) override;

private:
	FClientConnectionEvent ClientConnectedEvent;
	FClientConnectionEvent ClientDisconnectedEvent;
	TMap<UE::Cook::ECookOnTheFlyMessage, FHandleRequestDelegate> Handlers;
	TArray<ITargetPlatform*> ActiveTargetPlatforms;
};
