// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlatformProtocolServer.h"
#include "HAL/RunnableThread.h"
#include "Misc/OutputDeviceRedirector.h"
#include "NetworkMessage.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetDevice.h"
#include "Interfaces/ITargetDeviceSocket.h"
#include "GenericPlatform/GenericPlatformHostCommunication.h"



namespace
{
	class FSimpleAbstractSocket_PlatformProtocol : public FSimpleAbstractSocket
	{
	public:

		FSimpleAbstractSocket_PlatformProtocol(ITargetDeviceSocketPtr InSocket)
			: Socket(InSocket)
		{
			check(Socket != nullptr);
		}

		virtual bool Receive(uint8* Results, int32 Size) const override
		{
			return Socket->Receive(Results, Size);
		}

		virtual bool Send(const uint8* Buffer, int32 Size) const override
		{
			return Socket->Send(Buffer, Size);
		}

		virtual uint32 GetMagic() const override
		{
			return 0x9E2B83C7;
		}

	private:

		ITargetDeviceSocketPtr Socket;
	};
}


class FCookOnTheFlyServerPlatformProtocol::FConnectionThreaded
	: public FCookOnTheFlyClientConnectionBase
{
public:

	FConnectionThreaded(FCookOnTheFlyServerPlatformProtocol& InOwner, ITargetDevicePtr InDevice, ITargetDeviceSocketPtr InSocket)
		: FCookOnTheFlyClientConnectionBase(InOwner)
		, Device(InDevice)
		, Socket(InSocket)
	{
	}
	
	virtual ~FConnectionThreaded()
	{
		Device->CloseConnection(Socket);
		Socket = nullptr;
	}


	virtual void OnInit() override
	{
#if PLATFORM_WINDOWS
		FWindowsPlatformMisc::CoInitialize(ECOMModel::Multithreaded);
#endif
	}

	virtual void OnExit() override
	{
#if PLATFORM_WINDOWS
		FWindowsPlatformMisc::CoUninitialize();
#endif
	}

	virtual bool ReceivePayload(FArrayReader& Payload) override
	{
		return FNFSMessageHeader::ReceivePayload(Payload, FSimpleAbstractSocket_PlatformProtocol(Socket));
	}

	virtual bool SendPayload(const TArray<uint8>& Out) override
	{
		return FNFSMessageHeader::WrapAndSendPayload(Out, FSimpleAbstractSocket_PlatformProtocol(Socket));
	}

	FString GetName() const
	{
		return FString::Printf(TEXT("%s (%s)"), *Device->GetName(), *Device->GetPlatformControls().PlatformName());
	}

	ITargetDevicePtr GetDevice() const
	{
		return Device;
	}

private:
	ITargetDevicePtr	   Device;
	ITargetDeviceSocketPtr Socket;
};


FCookOnTheFlyServerPlatformProtocol::FCookOnTheFlyServerPlatformProtocol(const TArray<ITargetPlatform*>& InTargetPlatforms)
	: FCookOnTheFlyNetworkServerBase(InTargetPlatforms)
	, TargetPlatforms(InTargetPlatforms)
{
	Running = false;
	StopRequested = false;
}

bool FCookOnTheFlyServerPlatformProtocol::Start()
{
	UE_LOGF(LogCookOnTheFlyNetworkServer, Display, "Unreal Network File Server (custom protocol) starting up...");

	// Check the list of platforms once on start (any missing platforms will be ignored later on to avoid spamming the log).
	for (ITargetPlatform* TargetPlatform : TargetPlatforms)
	{
		if (!TargetPlatform->SupportsFeature(ETargetPlatformFeatures::DirectDataExchange))
		{
			UE_LOGF(LogCookOnTheFlyNetworkServer, Error, "Platform '%ls' does not support direct communication with targets (it will be ignored).", *TargetPlatform->PlatformName());
		}
	}

	// Create a thread that will be updating the list of connected target devices.
	Thread = FRunnableThread::Create(this, TEXT("FNetworkFileServerCustomProtocol"), 8 * 1024, TPri_AboveNormal);

	return true;
}


FCookOnTheFlyServerPlatformProtocol::~FCookOnTheFlyServerPlatformProtocol()
{
	// Kill the running thread.
	if (Thread != nullptr)
	{
		Thread->Kill(true);

		delete Thread;
		Thread = nullptr;
	}
}


uint32 FCookOnTheFlyServerPlatformProtocol::Run()
{
#if PLATFORM_WINDOWS
	FWindowsPlatformMisc::CoInitialize(ECOMModel::Multithreaded);
#endif

	Running = true;

	// Go until requested to be done.
	while (!StopRequested)
	{
		UpdateConnections();

		FPlatformProcess::Sleep(1.0f);
	}

#if PLATFORM_WINDOWS
	FWindowsPlatformMisc::CoUninitialize();
#endif

	return 0;
}


void FCookOnTheFlyServerPlatformProtocol::Stop()
{
	StopRequested = true;
}


void FCookOnTheFlyServerPlatformProtocol::Exit()
{
	// Close all connections.
	for (auto* Connection : Connections)
	{
		delete Connection;
	}

	Connections.Empty();
}


FString FCookOnTheFlyServerPlatformProtocol::GetSupportedProtocol() const
{
	return FString("custom");
}


bool FCookOnTheFlyServerPlatformProtocol::GetAddressList(TArray<TSharedPtr<FInternetAddr> >& OutAddresses) const
{
	return true;
}


bool FCookOnTheFlyServerPlatformProtocol::IsReadyToAcceptConnections() const
{
	return Running;
}


int32 FCookOnTheFlyServerPlatformProtocol::NumConnections() const
{
	return Connections.Num();
}

void FCookOnTheFlyServerPlatformProtocol::UpdateConnections()
{
	RemoveClosedConnections();

	AddConnectionsForNewDevices();
}


void FCookOnTheFlyServerPlatformProtocol::RemoveClosedConnections()
{
	for (int32 ConnectionIndex = 0; ConnectionIndex < Connections.Num(); ++ConnectionIndex)
	{
		FConnectionThreaded* Connection = Connections[ConnectionIndex];

		if (!Connection->IsRunning())
		{
			UE_LOGF(LogCookOnTheFlyNetworkServer, Display, "Client %ls disconnected.", *Connection->GetName());
			Connections.RemoveAtSwap(ConnectionIndex);
			delete Connection;
		}
	}
}


void FCookOnTheFlyServerPlatformProtocol::AddConnectionsForNewDevices()
{
	for (ITargetPlatform* TargetPlatform : TargetPlatforms)
	{
		if (TargetPlatform->SupportsFeature(ETargetPlatformFeatures::DirectDataExchange))
		{
			AddConnectionsForNewDevices(TargetPlatform);
		}
	}
}


void FCookOnTheFlyServerPlatformProtocol::AddConnectionsForNewDevices(ITargetPlatform* TargetPlatform)
{
	TArray<ITargetDevicePtr> TargetDevices;

	TargetPlatform->RefreshAllDevices();
	TargetPlatform->GetAllDevices(TargetDevices);

	for (ITargetDevicePtr Device : TargetDevices)
	{
		if (Device->IsConnected())
		{
			FConnectionThreaded** ExistingConnection =
				Connections.FindByPredicate(
					[&Device](const FConnectionThreaded* Connection)
					{
						return Connection->GetDevice() == Device;
					}
			);

			// Checking IsProtocolAvailable first would make more sense, but internally it queries
			// COM interfaces, which throws exceptions if the protocol is already in use. While we catch
			// and process these exceptions, Visual Studio intercepts then as well and outputs messages
			// spamming the log, which hinders the debugging experience.
			if (!ExistingConnection)
			{
				if (Device->IsProtocolAvailable(EHostProtocol::CookOnTheFly))
				{
					ITargetDeviceSocketPtr Socket = Device->OpenConnection(EHostProtocol::CookOnTheFly);

					if (Socket)
					{
						if (Socket->Connected())
						{
							FConnectionThreaded* Connection = new FConnectionThreaded(*this, Device, Socket);
							if (Connection->Initialize())
							{
								Connections.Add(Connection);
								UE_LOGF(LogCookOnTheFlyNetworkServer, Display, "Client %ls connected.", *Connection->GetName());
							}
							else
							{
								delete Connection;
							}
						}
						else
						{
							Device->CloseConnection(Socket);
						}
					}
				}
			}
		}
	}
}
