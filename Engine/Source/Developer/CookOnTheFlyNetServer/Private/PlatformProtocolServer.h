// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Runnable.h"
#include "CookOnTheFlyNetServerBase.h"

class ITargetPlatform;


/**
 * This class wraps the server thread and network connection.
 *
 * It uses ITargetDeviceSocket to exchanging data with the connected targets.
 * This interface is an abstraction for direct pc-target communication as provided
 * by the platforms. IPlatformHostCommunication/IPlatformHostSocket are the
 * corresponding interfaces used on the game side.
 *
 * This implementation is based on FNetworkFileServer (the TCP-based server).
 */
class FCookOnTheFlyServerPlatformProtocol
	: public FRunnable
	, public FCookOnTheFlyNetworkServerBase
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InFileServerOptions Network file server options
	 */
	FCookOnTheFlyServerPlatformProtocol(const TArray<ITargetPlatform*>& TargetPlatforms);

	/**
	 * Destructor.
	 */
	~FCookOnTheFlyServerPlatformProtocol();

public:

	//~ Begin FRunnable Interface

	virtual uint32 Run()  override;
	virtual void   Stop() override;
	virtual void   Exit() override;

	//~ End FRunnable Interface

public:

	//~ Begin ICookOnTheFlyNetworkServer Interface

	virtual bool    IsReadyToAcceptConnections() const override;
	virtual bool    GetAddressList(TArray<TSharedPtr<FInternetAddr> >& OutAddresses) const override;
	virtual FString GetSupportedProtocol() const override;
	virtual int32   NumConnections() const override;
	virtual bool    Start() override;
	//~ End ICookOnTheFlyNetworkServer Interface

private:

	void UpdateConnections();
	void AddConnectionsForNewDevices();
	void AddConnectionsForNewDevices(ITargetPlatform* TargetPlatform);
	void RemoveClosedConnections();

	class FConnectionThreaded;

	// File server options
	TArray<ITargetPlatform*> TargetPlatforms;

	// Holds the server thread object.
	FRunnableThread* Thread;

	// Holds the list of all client connections.
	TArray<FConnectionThreaded*> Connections;

	// Holds a flag indicating whether the thread should stop executing
	std::atomic<bool> StopRequested;

	// Is the Listener thread up and running. 
	std::atomic<bool> Running;
};
