// Copyright Epic Games, Inc. All Rights Reserved.

#include "TCPServer.h"
#include "HAL/RunnableThread.h"
#include "Misc/OutputDeviceRedirector.h"
#include "IPAddress.h"
#include "Sockets.h"
#include "SocketSubsystem.h"
#include "NetworkMessage.h"

class FCookOnTheFlyClientConnectionTCP
	: public FCookOnTheFlyClientConnectionBase
{
public:

	FCookOnTheFlyClientConnectionTCP(FCookOnTheFlyServerTCP& InOwner, FSocket* InSocket)
		: FCookOnTheFlyClientConnectionBase(InOwner)
		, Socket(InSocket)
	{
	}

	~FCookOnTheFlyClientConnectionTCP()
	{
		ISocketSubsystem::Get()->DestroySocket(Socket);
	}

	virtual bool ReceivePayload(FArrayReader& Payload) override
	{
		return FNFSMessageHeader::ReceivePayload(Payload, FSimpleAbstractSocket_FSocket(Socket));
	}

	virtual bool SendPayload(const TArray<uint8>& Payload) override
	{
		return FNFSMessageHeader::WrapAndSendPayload(Payload, FSimpleAbstractSocket_FSocket(Socket));
	}

	virtual void OnExit() override
	{
		Socket->Close();
	}

	void GetAddress(FInternetAddr& Addr)
	{
		Socket->GetAddress(Addr);
	}

	void GetPeerAddress(FInternetAddr& Addr)
	{
		Socket->GetPeerAddress(Addr);
	}

private:
	FSocket* Socket;
};



/* FCookOnTheFlyServerTCP constructors
 *****************************************************************************/

FCookOnTheFlyServerTCP::FCookOnTheFlyServerTCP(int32 InPort, const TArray<ITargetPlatform*>& InTargetPlatforms)
	: FCookOnTheFlyNetworkServerBase(InTargetPlatforms)
	, Port(InPort)
{
	if (Port < 0)
	{
		Port = DEFAULT_TCP_FILE_SERVING_PORT;
	}

	Running.Set(false);
	StopRequested.Set(false);
}

FCookOnTheFlyServerTCP::~FCookOnTheFlyServerTCP()
{
	// Kill the running thread.
	if (Thread != NULL)
	{
		Thread->Kill(true);

		delete Thread;
		Thread = NULL;
	}

	// We are done with the socket.
	Socket->Close();
	ISocketSubsystem::Get()->DestroySocket(Socket);
	Socket = NULL;
}

bool FCookOnTheFlyServerTCP::Start()
{
	// make sure sockets are going
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();

	if (!SocketSubsystem)
	{
		UE_LOGF(LogCookOnTheFlyNetworkServer, Error, "Could not get socket subsystem.");
	}
	else
	{
		// listen on any IP address
		ListenAddr = SocketSubsystem->GetLocalBindAddr(*GLog);
		ListenAddr->SetPort(Port);

		// create a server TCP socket
		Socket = SocketSubsystem->CreateSocket(NAME_Stream, TEXT("FCookOnTheFlyServerTCP-listen"), ListenAddr->GetProtocolType());

		if (!Socket)
		{
			UE_LOGF(LogCookOnTheFlyNetworkServer, Error, "Could not create listen socket.");
		}
		else
		{
			Socket->SetReuseAddr();
			Socket->SetNoDelay();

			// bind to the address
			if (!Socket->Bind(*ListenAddr))
			{
				UE_LOGF(LogCookOnTheFlyNetworkServer, Warning, "Failed to bind listen socket %ls in FCookOnTheFlyServerTCP", *ListenAddr->ToString(true));
			}
			// listen for connections
			else if (!Socket->Listen(16))
			{
				UE_LOGF(LogCookOnTheFlyNetworkServer, Warning, "Failed to listen on socket %ls in FCookOnTheFlyServerTCP", *ListenAddr->ToString(true));
			}
			else
			{
				// set the port on the listen address to be the same as the port on the socket
				int32 port = Socket->GetPortNo();
				check((Port == 0 && port != 0) || port == Port);
				ListenAddr->SetPort(port);

				// now create a thread to accept connections
				Thread = FRunnableThread::Create(this, TEXT("FCookOnTheFlyServerTCP"), 8 * 1024, TPri_AboveNormal);
				return true;
			}
		}
	}
	return false;
}

/* FRunnable overrides
 *****************************************************************************/

uint32 FCookOnTheFlyServerTCP::Run()
{
	Running.Set(true);
	// go until requested to be done
	while (!StopRequested.GetValue())
	{
		bool bReadReady = false;

		// clean up closed connections
		for (int32 ConnectionIndex = 0; ConnectionIndex < Connections.Num(); ++ConnectionIndex)
		{
			FCookOnTheFlyClientConnectionTCP* Connection = Connections[ConnectionIndex];

			if (!Connection->IsRunning())
			{
				UE_LOGF(LogCookOnTheFlyNetworkServer, Display, "Client disconnected.");
				Connections.RemoveAtSwap(ConnectionIndex);
				delete Connection;
			}
		}

		// check for incoming connections
		if (Socket->WaitForPendingConnection(bReadReady, FTimespan::FromSeconds(0.25f)))
		{
			if (bReadReady)
			{
				FSocket* ClientSocket = Socket->Accept(TEXT("Remote Console Connection"));

				if (ClientSocket != NULL)
				{
					TSharedPtr<FInternetAddr> Addr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();
					ClientSocket->GetAddress(*Addr);
					TSharedPtr<FInternetAddr> PeerAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();
					ClientSocket->GetPeerAddress(*PeerAddr);

					for (auto PreviousConnection : Connections)
					{
						TSharedPtr<FInternetAddr> PreviousAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();;
						PreviousConnection->GetAddress(*PreviousAddr);
						TSharedPtr<FInternetAddr> PreviousPeerAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();;
						PreviousConnection->GetPeerAddress(*PreviousPeerAddr);
						if ((*Addr == *PreviousAddr) &&
							(*PeerAddr == *PreviousPeerAddr))
						{
							// kill the connection 
							PreviousConnection->Disconnect();
							UE_LOGF(LogCookOnTheFlyNetworkServer, Warning, "Killing client connection because new client connected from same address.");
						}
					}

					FCookOnTheFlyClientConnectionTCP* Connection = new FCookOnTheFlyClientConnectionTCP(*this, ClientSocket);
					if (Connection->Initialize())
					{
						Connections.Add(Connection);
						UE_LOGF(LogCookOnTheFlyNetworkServer, Display, "Client connected.");
					}
					else
					{
						delete Connection;
					}
				}
			}
		}
		else
		{
			FPlatformProcess::Sleep(0.25f);
		}
	}

	return 0;
}


void FCookOnTheFlyServerTCP::Exit()
{
	// close all connections
	for (int32 ConnectionIndex = 0; ConnectionIndex < Connections.Num(); ConnectionIndex++)
	{

		delete Connections[ConnectionIndex];
	}

	Connections.Empty();
}


/* ICookOnTheFlyNetworkServer overrides
 *****************************************************************************/

FString FCookOnTheFlyServerTCP::GetSupportedProtocol() const
{
	return FString("tcp");
}


bool FCookOnTheFlyServerTCP::GetAddressList(TArray<TSharedPtr<FInternetAddr> >& OutAddresses) const
{
	if (ListenAddr.IsValid())
	{
		FString ListenAddressString = ListenAddr->ToString(true);

		if (ListenAddressString.StartsWith(TEXT("0.0.0.0")))
		{
			if (ISocketSubsystem::Get()->GetLocalAdapterAddresses(OutAddresses))
			{
				for (int32 AddressIndex = 0; AddressIndex < OutAddresses.Num(); ++AddressIndex)
				{
					OutAddresses[AddressIndex]->SetPort(ListenAddr->GetPort());
				}
			}
		}
		else
		{
			OutAddresses.Add(ListenAddr);
		}
	}

	return (OutAddresses.Num() > 0);
}


bool FCookOnTheFlyServerTCP::IsReadyToAcceptConnections(void) const
{
	return (Running.GetValue() != 0);
}

int32 FCookOnTheFlyServerTCP::NumConnections(void) const
{
	return Connections.Num();
}
