// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/Runnable.h"
#include "CookOnTheFlyNetServerBase.h"

class FSocket;


/**
 * This class wraps the server thread and network connection
 */
class FCookOnTheFlyServerTCP
	: public FRunnable
	, public FCookOnTheFlyNetworkServerBase
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InFileServerOptions Network file server options
	 */
	FCookOnTheFlyServerTCP(int32 InPort, const TArray<ITargetPlatform*>& InTargetPlatforms);

	/**
	 * Destructor.
	 */
	~FCookOnTheFlyServerTCP();

public:

	// FRunnable Interface

	virtual bool Init() override
	{
		return true;
	}

	virtual uint32 Run() override;

	virtual void Stop() override
	{
		StopRequested.Set(true);
	}

	virtual void Exit() override;

	// ICookOnTheFlyNetworkServer interface

	virtual bool IsReadyToAcceptConnections(void) const override;
	virtual bool GetAddressList(TArray<TSharedPtr<FInternetAddr>>& OutAddresses) const override;
	virtual FString GetSupportedProtocol() const override;
	virtual int32 NumConnections() const override;
	virtual bool Start() override;

private:
	int32 Port;

	// Holds the server (listening) socket.
	FSocket* Socket;

	// Holds the server thread object.
	FRunnableThread* Thread;

	// Holds the list of all client connections.
	TArray<class FCookOnTheFlyClientConnectionTCP*> Connections;

	// Holds a flag indicating whether the thread should stop executing
	FThreadSafeCounter StopRequested;

	// Is the Listner thread up and running. 
	FThreadSafeCounter Running;

	// Holds the address that the server is bound to.
	TSharedPtr<FInternetAddr> ListenAddr;
};
